// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeToolset.h"

#include "Modules/ModuleManager.h"

/*virtual*/ void FStateTreeToolsetModule::StartupModule()
{
}

/*virtual*/ void FStateTreeToolsetModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FStateTreeToolsetModule, StateTreeToolset);
DEFINE_LOG_CATEGORY(LogStateTreeToolset);
