// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StateTreeToolset : ModuleRules
{
	public StateTreeToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		PublicDependencyModuleNames.Add("Core");
	}
}
